{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Adjust Analytics Provider",
	"Description" : "Adjust Analytics Provider",
	"Category" : "Analytics",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "https://docs.unrealengine.com/latest/INT/Gameplay/Analytics/index.html",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "AdjustEditor",
			"Type" : "Editor",
			"LoadingPhase" : "Default"
		},
		{
			"Name" : "AndroidAdjust",
			"Type" : "Runtime",
			"LoadingPhase" : "PreDefault",
			"PlatformAllowList" :
			[
				"Android"
			]
		},
		{
			"Name" : "IOSAdjust",
			"Type" : "Runtime",
			"LoadingPhase" : "PreDefault",
			"PlatformAllowList" :
			[
				"IOS"
			]
		}
	]
}