// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnalyticsSettings.h"
#include "AdjustSettings.generated.h"

UENUM()
namespace EAndroidAdjustLogging
{
	enum Type : int
	{
		/** enable all logging */
		verbose  UMETA(DisplayName = "VERBOSE"),
		/** enable more logging */
		debug    UMETA(DisplayName = "DEBUG"),
		/** the default */
		info    UMETA(DisplayName = "INFO"),
		/** disable info logging */
		warn     UMETA(DisplayName = "WARN"),
		/** disable warnings as well */
		error    UMETA(DisplayName = "ERROR"),
		/** disable errors as well */
		assert   UMETA(DisplayName = "ASSERT"),
		/** disable all log output */
		supress  UMETA(DisplayName = "SUPRESS"),
	};
}

/**
 * Holds the event name and corresponding token from Adjust dashboard.
 */
USTRUCT()
struct FAdjustEventMapping
{
	GENERATED_USTRUCT_BODY()

	/** The event name (the one passed in to RecordEvent calls). */
	UPROPERTY(EditAnywhere, Category = AdjustEvents)
	FString Name;

	/** The token of the corresponding event, generated by the Adjust dashboard. */
	UPROPERTY(EditAnywhere, Category = AdjustEvents)
	FString Token;
};

UCLASS()
class UAdjustSettings
	: public UAnalyticsSettingsBase
{
	GENERATED_UCLASS_BODY()

	// Check to run in sandbox mode instead of production for non-distribution builds
	UPROPERTY(EditAnywhere, Category=Adjust, meta=(ConfigRestartRequired = true, DisplayName = "Sandbox mode for non-distribution?"))
	bool bSandboxNondistribution;

	// Check to run in sandbox mode instead of production for distribution builds
	UPROPERTY(EditAnywhere, Category=Adjust, meta=(ConfigRestartRequired = true, DisplayName = "Sandbox mode for distribution?"))
	bool bSandboxDistribution;

	// Application token from dashboard
	UPROPERTY(EditAnywhere, Category=Adjust, meta=(ConfigRestartRequired = true, DisplayName = "Application token"))
	FString AppToken;

	// Level of verbosity to use for logging
	UPROPERTY(EditAnywhere, Category=Adjust, meta=(DisplayName = "Logging level"))
	TEnumAsByte<EAndroidAdjustLogging::Type> LogLevel;

	// Token from dashboard for tracker URL (may be left empty) 
	UPROPERTY(EditAnywhere, Category=Adjust, meta=(DisplayName = "Default tracker token"))
	FString DefaultTracker;

	// Override the process name; will use the package name if not provided
	UPROPERTY(EditAnywhere, Category=Adjust, meta=(DisplayName = "Process name (optional)"))
	FString ProcessName;

	// Optional event buffering (send in batches once a minute instead of immediately)
	UPROPERTY(EditAnywhere, Category=Adjust, meta=(DisplayName = "Enable event buffering"))
	bool bEventBuffering;
	
	// Send while in application in background
	UPROPERTY(EditAnywhere, Category=Adjust, meta=(DisplayName = "Send while in background"))
	bool bSendInBackground;

	// Optional start delay (up to 10 seconds) before first events are sent
	UPROPERTY(EditAnywhere, Category=Adjust, meta=(DisplayName = "Delay start (seconds)"))
	float DelayStart;

	// Mapping of event names to tokens generated by Adjust dashboard.
	UPROPERTY(EditAnywhere, Category = AdjustEvents)
	TArray<FAdjustEventMapping> EventMap;

// UAnalyticsSettingsBase interface
protected:
	/**
	 * Provides a mechanism to read the section based information into this UObject's properties
	 */
	virtual void ReadConfigSettings();
	/**
	 * Provides a mechanism to save this object's properties to the section based ini values
	 */
	virtual void WriteConfigSettings();
	
private:
	bool GetBoolFromConfig(const FString& IniName, const FString& SectionName, const FString& KeyName, bool bIsRequired, bool bDefault);
	void WriteConfigSection(const FString& Section);
};
