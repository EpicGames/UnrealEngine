// Copyright Epic Games, Inc. All Rights Reserved.

#include "AdjustEditor.h"


#include "AdjustSettings.h"
#include "Analytics.h"

IMPLEMENT_MODULE( FAdjustEditorModule, AdjustEditor );

#define LOCTEXT_NAMESPACE "Adjust"

void FAdjustEditorModule::StartupModule()
{
}

void FAdjustEditorModule::ShutdownModule()
{
}

UAdjustSettings::UAdjustSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SettingsDisplayName = LOCTEXT("SettingsDisplayName", "Adjust");
	SettingsTooltip = LOCTEXT("SettingsTooltip", "Adjust configuration settings");
}

bool UAdjustSettings::GetBoolFromConfig(const FString& IniName, const FString& InSectionName, const FString& KeyName, bool bIsRequired, bool bDefault)
{
	FString Result = FAnalytics::Get().GetConfigValueFromIni(IniName, InSectionName, KeyName, bIsRequired);
	if (Result.Len() == 0)
	{
		return bDefault;
	}
	Result = Result.ToLower();
	return Result.Equals(TEXT("true")) || Result.Equals(TEXT("yes"));
}

void UAdjustSettings::ReadConfigSettings()
{
	// Just read the release section since we write same values to all sections
	bSandboxNondistribution = GetBoolFromConfig(GetIniName(), GetReleaseIniSection(), TEXT("AdjustSandboxNondistribution"), false, true);
	bSandboxDistribution = GetBoolFromConfig(GetIniName(), GetReleaseIniSection(), TEXT("AdjustSandboxDistribution"), false, false);
	AppToken = FAnalytics::Get().GetConfigValueFromIni(GetIniName(), GetReleaseIniSection(), TEXT("AdjustAppToken"), true);

	FString LogLevelIn = FAnalytics::Get().GetConfigValueFromIni(GetIniName(), GetReleaseIniSection(), TEXT("AdjustLogLevel"), false);
	if (LogLevelIn.Equals("VERBOSE"))
		LogLevel = EAndroidAdjustLogging::verbose;
	else if (LogLevelIn.Equals("DEBUG"))
		LogLevel = EAndroidAdjustLogging::debug;
	else if (LogLevelIn.Equals("INFO"))
		LogLevel = EAndroidAdjustLogging::info;
	else if (LogLevelIn.Equals("WARN"))
		LogLevel = EAndroidAdjustLogging::warn;
	else if (LogLevelIn.Equals("ERROR"))
		LogLevel = EAndroidAdjustLogging::error;
	else if (LogLevelIn.Equals("ASSERT"))
		LogLevel = EAndroidAdjustLogging::assert;
	else if (LogLevelIn.Equals("SUPRESS"))
		LogLevel = EAndroidAdjustLogging::supress;
	else LogLevel = EAndroidAdjustLogging::info;

	DefaultTracker = FAnalytics::Get().GetConfigValueFromIni(GetIniName(), GetReleaseIniSection(), TEXT("AdjustDefaultTracker"), false);
	ProcessName = FAnalytics::Get().GetConfigValueFromIni(GetIniName(), GetReleaseIniSection(), TEXT("AdjustProcessName"), false);
	bEventBuffering = GetBoolFromConfig(GetIniName(), GetReleaseIniSection(), TEXT("AdjustEventBuffering"), false, false);
	bSendInBackground = GetBoolFromConfig(GetIniName(), GetReleaseIniSection(), TEXT("AdjustSendInBackground"), false, false);

	FString Result = FAnalytics::Get().GetConfigValueFromIni(GetIniName(), GetReleaseIniSection(), TEXT("AdjustDelayStart"), false);
	if (Result.Len() == 0)
	{
		Result = "0.0";
	}
	DelayStart = FCString::Atof(*Result);

	// For now read event mapping from own section
	EventMap.Empty();
	TArray<FString> EventNames;
	TArray<FString> EventTokens;
	int NameCount = GConfig->GetArray(TEXT("AdjustAnalyticsEventMapping"), TEXT("EventNames"), EventNames, GetIniName());
	int TokenCount = GConfig->GetArray(TEXT("AdjustAnalyticsEventMapping"), TEXT("EventTokens"), EventTokens, GetIniName());
	int Count = NameCount <= TokenCount ? NameCount : TokenCount;
	for (int Index = 0; Index < Count; ++Index)
	{
		FAdjustEventMapping Mapping;
		Mapping.Name = EventNames[Index];
		Mapping.Token = EventTokens[Index];
		EventMap.Add(Mapping);
	}
}

void UAdjustSettings::WriteConfigSettings()
{
	// Just write the same values to all configurations for now (we don't care which one we read)
	WriteConfigSection(GetReleaseIniSection());
	WriteConfigSection(GetTestIniSection());
	WriteConfigSection(GetDebugIniSection());
	WriteConfigSection(GetDevelopmentIniSection());

	// For now write the event mapping to own section manually here
	TArray<FString> EventNames;
	TArray<FString> EventTokens;
	for (FAdjustEventMapping Entry : EventMap)
	{
		EventNames.Add(Entry.Name);
		EventTokens.Add(Entry.Token);
	}
	GConfig->FConfigCacheIni::SetArray(TEXT("AdjustAnalyticsEventMapping"), TEXT("EventNames"), EventNames, GetIniName());
	GConfig->FConfigCacheIni::SetArray(TEXT("AdjustAnalyticsEventMapping"), TEXT("EventTokens"), EventTokens, GetIniName());
}

void UAdjustSettings::WriteConfigSection(const FString& Section)
{
	FAnalytics::Get().WriteConfigValueToIni(GetIniName(), Section, TEXT("AdjustSandboxNondistribution"), bSandboxNondistribution ? TEXT("true") : TEXT("false"));
	FAnalytics::Get().WriteConfigValueToIni(GetIniName(), Section, TEXT("AdjustSandboxDistribution"), bSandboxDistribution ? TEXT("true") : TEXT("false"));
	FAnalytics::Get().WriteConfigValueToIni(GetIniName(), Section, TEXT("AdjustAppToken"), AppToken);

	FString LogLevelOut = TEXT("INFO");
	switch (LogLevel)
	{
		case EAndroidAdjustLogging::verbose: LogLevelOut = TEXT("VERBOSE"); break;
		case EAndroidAdjustLogging::debug: LogLevelOut = TEXT("DEBUG"); break;
		case EAndroidAdjustLogging::info: LogLevelOut = TEXT("INFO"); break;
		case EAndroidAdjustLogging::warn: LogLevelOut = TEXT("WARN"); break;
		case EAndroidAdjustLogging::error: LogLevelOut = TEXT("ERROR"); break;
		case EAndroidAdjustLogging::assert: LogLevelOut = TEXT("ASSERT"); break;
		case EAndroidAdjustLogging::supress: LogLevelOut = TEXT("SUPRESS"); break;
	}
	FAnalytics::Get().WriteConfigValueToIni(GetIniName(), Section, TEXT("AdjustLogLevel"), LogLevelOut);

	FAnalytics::Get().WriteConfigValueToIni(GetIniName(), Section, TEXT("AdjustDefaultTracker"), DefaultTracker);
	FAnalytics::Get().WriteConfigValueToIni(GetIniName(), Section, TEXT("AdjustProcessName"), ProcessName);
	FAnalytics::Get().WriteConfigValueToIni(GetIniName(), Section, TEXT("AdjustEventBuffering"), bEventBuffering ? TEXT("true") : TEXT("false"));
	FAnalytics::Get().WriteConfigValueToIni(GetIniName(), Section, TEXT("AdjustSendInBackground"), bSendInBackground ? TEXT("true") : TEXT("false"));
	if (DelayStart > 0.0f)
	{
		FAnalytics::Get().WriteConfigValueToIni(GetIniName(), Section, TEXT("AdjustDelayStart"), FString::SanitizeFloat(DelayStart));
	}
	else
	{
		FAnalytics::Get().WriteConfigValueToIni(GetIniName(), Section, TEXT("AdjustDelayStart"), TEXT("0.0"));
	}
}

#undef LOCTEXT_NAMESPACE
