<?xml version="1.0" encoding="utf-8"?>
<root xmlns:android="http://schemas.android.com/apk/res/android">
	<!-- init section is always evaluated once per architecture -->
	<init>
		<log text="Adjust Analytics init"/>

		<!-- Must be shipping and for distribution (allow Development distribution for testing) -->
		<setBoolIsEqual result="bShipping" arg1="$S(Configuration)" arg2="Shipping"/>
		<setBoolAnd result="bShippingAndDistribution" arg1="$B(bShipping)" arg2="$B(Distribution)"/>
		
		<!-- Check if sandbox environment based on configuration and distribution state -->
		<if condition="bShippingAndDistribution">
			<true>
				<setBoolFromProperty result="bSandbox" ini="Engine" section="Analytics" property="bAdjustSandboxDistribution" default="false"/>
			</true>
			<false>
				<setBoolFromProperty result="bSandbox" ini="Engine" section="Analytics" property="bAdjustSandboxNondistribution" default="true"/>
			</false>
		</if>
		
		<!-- Grab the other settings -->
		<setStringFromProperty result="AppToken" ini="Engine" section="Analytics" property="AdjustAppToken" default="NOT SET"/>
		<setStringFromProperty result="LogLevel" ini="Engine" section="Analytics" property="AdjustLogLevel" default="INFO"/>
		<setStringFromProperty result="DefaultTracker" ini="Engine" section="Analytics" property="AdjustDefaultTracker" default=""/>
		<setStringFromProperty result="ProcessName" ini="Engine" section="Analytics" property="AdjustProcessName" default=""/>
		<setBoolFromProperty result="bEventBuffering" ini="Engine" section="Analytics" property="AdjustEventBuffering" default="false"/>
		<setBoolFromProperty result="bSendInBackground" ini="Engine" section="Analytics" property="AdjustSendInBackground" default="false"/>
		<setStringFromProperty result="DelayStart" ini="Engine" section="Analytics" property="AdjustDelayStart" default="0.0"/>

		<!-- Determine if these settings were filled in -->
		<setBoolIsEqual result="bNoDefaultTracker" arg1="$S(DefaultTracker)" arg2=""/>
		<setBoolIsEqual result="bNoProcessName" arg1="$S(ProcessName)" arg2=""/>
		<setBoolIsEqual result="bNoDelayStart" arg1="$S(DelayStart)" arg2="0.0"/>
	</init>
	
	<!-- optional files or directories to copy to Intermediate/Android/APK -->
	<resourceCopies>
	</resourceCopies>
	
	<!-- Files to copy before the build so they can be used in the build -->
	<prebuildCopies>
		<log text="Copying Adjust buildtime files to staging"/>
		<copyDir src="$S(PluginDir)/ThirdParty/adjust_library"
					dst = "$S(BuildDir)/JavaLibs/adjust_library" />
	</prebuildCopies>

	<buildGradleAdditions>
		<insert>
dependencies {
	implementation('com.google.android.gms:play-services-analytics:18.0.2')
}
		</insert>
	</buildGradleAdditions>
	
	<!-- optional additions to proguard -->
	<proguardAdditions>
		<insert>
-keepclassmembers enum * {
    public static **[] values();
    public static ** valueOf(java.lang.String);
}
-keep class com.adjust.sdk.plugin.MacAddressUtil {
    java.lang.String getMacAddress(android.content.Context);
}
-keep class com.adjust.sdk.plugin.AndroidIdUtil {
    java.lang.String getAndroidId(android.content.Context);
}
-keep class com.google.android.gms.common.ConnectionResult {
    int SUCCESS;
}
-keep class com.google.android.gms.ads.identifier.AdvertisingIdClient {
    com.google.android.gms.ads.identifier.AdvertisingIdClient$Info getAdvertisingIdInfo(android.content.Context);
}
-keep class com.google.android.gms.ads.identifier.AdvertisingIdClient$Info {
    java.lang.String getId();
    boolean isLimitAdTrackingEnabled();
}
		</insert>
	</proguardAdditions>
  
	<!-- optional updates applied to AndroidManifest.xml -->
	<androidManifestUpdates>
		<!-- Add permissions -->
		<addPermission android:name="android.permission.INTERNET" />
		<addPermission android:name="android.permission.ACCESS_WIFI_STATE" />
		
		<!-- Add receiver -->
		<addElements tag="application">
			<receiver
				android:name="com.adjust.sdk.AdjustReferrerReceiver"
				android:exported="true">
				<intent-filter>
					<action android:name="com.android.vending.INSTALL_REFERRER" />
				</intent-filter>
			</receiver>
		</addElements>
	</androidManifestUpdates>

	<gameActivityImportAdditions>
		<insert>
import java.util.LinkedHashMap;
import java.util.Map;
import android.net.Uri;
import com.adjust.sdk.Adjust;
import com.adjust.sdk.AdjustAttribution;
import com.adjust.sdk.AdjustConfig;
import com.adjust.sdk.AdjustEvent;
import com.adjust.sdk.AdjustEventFailure;
import com.adjust.sdk.AdjustEventSuccess;
import com.adjust.sdk.LogLevel;
import com.adjust.sdk.OnAttributionChangedListener;
import com.adjust.sdk.OnDeeplinkResponseListener;
import com.adjust.sdk.OnEventTrackingFailedListener;
import com.adjust.sdk.OnEventTrackingSucceededListener;
import com.adjust.sdk.OnSessionTrackingFailedListener;
import com.adjust.sdk.OnSessionTrackingSucceededListener;
import com.adjust.sdk.AdjustSessionFailure;
import com.adjust.sdk.AdjustSessionSuccess;
		</insert>
	</gameActivityImportAdditions>
  
	<gameActivityClassAdditions>
		<insert>
	Map&lt;String, String&gt; Adjust_Event_CallbackParameters;
	Map&lt;String, String&gt; Adjust_Event_PartnerParameters;

	public void AndroidThunkJava_Adjust_SetEnabled(boolean Enable)
	{
		// Disable or enable tracking (REMEMBERED BETWEEEN SESSIONS!!)
		Adjust.setEnabled(Enable);
	}

	public void AndroidThunkJava_Adjust_SetOfflineMode(boolean Offline)
	{
		// Offline mode: (will save to file and send when disabled)  ALWAYS STARTS in online mode!
		Adjust.setOfflineMode(Offline);
	}
		
	public void AndroidThunkJava_Adjust_SetPushToken(String Token)
	{
		Adjust.setPushToken(Token);
	}

	public void AndroidThunkJava_Adjust_AddSessionPartnerParameter(String Key, String Value)
	{
		Adjust.addSessionPartnerParameter(Key, Value);
	}

	public void AndroidThunkJava_Adjust_RemoveSessionPartnerParameter(String Key)
	{
		Adjust.removeSessionPartnerParameter(Key);
	}
		
	public void AndroidThunkJava_Adjust_ResetSessionPartnerParameters()
	{
		Adjust.resetSessionPartnerParameters();
	}
		
	public void AndroidThunkJava_Adjust_Event_AddCallbackParameter(String Key, String Value)
	{
		if (Adjust_Event_CallbackParameters == null)
		{
			// use LinkedHashMap to preserve the ordering
			Adjust_Event_CallbackParameters = new LinkedHashMap&lt;String, String&gt;();
		}
		Adjust_Event_CallbackParameters.put(Key, Value);
	}

	public void AndroidThunkJava_Adjust_Event_RemoveCallbackParameter(String Key)
	{
		if (Adjust_Event_CallbackParameters != null)
		{
			Adjust_Event_CallbackParameters.remove(Key);
		}
	}

	public void AndroidThunkJava_Adjust_Event_ResetCallbackParameters()
	{
		Adjust_Event_CallbackParameters = null;
	}

	public void AndroidThunkJava_Adjust_Event_AddPartnerParameter(String Key, String Value)
	{
		if (Adjust_Event_PartnerParameters == null)
		{
			// use LinkedHashMap to preserve the ordering
			Adjust_Event_PartnerParameters = new LinkedHashMap&lt;String, String&gt;();
		}
		Adjust_Event_PartnerParameters.put(Key, Value);
	}

	public void AndroidThunkJava_Adjust_Event_RemovePartnerParameter(String Key)
	{
		if (Adjust_Event_PartnerParameters != null)
		{
			Adjust_Event_PartnerParameters.remove(Key);
		}
	}

	public void AndroidThunkJava_Adjust_Event_ResetPartnerParameters()
	{
		Adjust_Event_PartnerParameters = null;
	}

	private AdjustEvent Adjust_CreateEvent(String Token)
	{
		AdjustEvent event = new AdjustEvent(Token);
		if (Adjust_Event_CallbackParameters != null)
		{
			for (LinkedHashMap.Entry&lt;String, String&gt; entry : Adjust_Event_CallbackParameters.entrySet())
			{
				event.addCallbackParameter(entry.getKey(), entry.getValue());
			}
		}
		if (Adjust_Event_PartnerParameters != null)
		{
			for (LinkedHashMap.Entry&lt;String, String&gt; entry : Adjust_Event_PartnerParameters.entrySet())
			{
				event.addPartnerParameter(entry.getKey(), entry.getValue());
			}
		}
		return event;
	}

	public void AndroidThunkJava_Adjust_SendEvent(String Token)
	{
		Adjust.trackEvent(Adjust_CreateEvent(Token));
	}

	public void AndroidThunkJava_Adjust_SendRevenueEvent(String Token, String OrderId, double Amount, String Currency)
	{
		AdjustEvent event = Adjust_CreateEvent(Token);
		event.setRevenue(Amount, Currency);  // 0.01, USD
		if (OrderId != "")
		{
			event.setOrderId(OrderId);  // optional to prevent duplication (remembers last 10)
		}
		Adjust.trackEvent(event);
	}
		</insert>
	</gameActivityClassAdditions>

	<gameActivityOnCreateAdditions>
		<insertValue value="        String appToken = &quot;$S(AppToken)&quot;;" />
		<insertNewline/>
		<if condition="bSandbox">
			<true>
				<insert>
		String environment = AdjustConfig.ENVIRONMENT_SANDBOX;
				</insert>
			</true>
			<false>
				<insert>
		String environment = AdjustConfig.ENVIRONMENT_PRODUCTION;
				</insert>
			</false>
		</if>
		<if condition="bNoDelayStart">
			<false>
				<insertValue value="        adjustConfig.setDelayStart($S(DelayStart));" />
			</false>
		</if>
		<insertNewline/>
		<insert>
		AdjustConfig config = new AdjustConfig(this, appToken, environment);
		</insert>
		<insertValue value="        config.setLogLevel(LogLevel.$S(LogLevel));" />
		<insertNewline/>
		<if condition="bSendInBackground">
			<true>
				<insert>
        config.setSendInBackground(true);
				</insert>
			</true>
			<false>
				<insert>
        config.setSendInBackground(false);
				</insert>
			</false>
		</if>
		<if condition="bEventBuffering">
			<true>
				<insert>
        config.setEventBufferingEnabled(true);
				</insert>
			</true>
			<false>
				<insert>
        config.setEventBufferingEnabled(false);
				</insert>
			</false>
		</if>

		<if condition="bNoDefaultTracker">
			<false>
				<insertValue value="        config.setDefaultTracker(&quot;$S(DefaultTracker)&quot;);"/>
				<insertNewline/>
			</false>
		</if>
		
		<if condition="bNoProcessName">
			<false>
				<insertValue value="        config.setProcessName(&quot;$S(ProcessName)&quot;);"/>
				<insertNewline/>
			</false>
		</if>
		
		<insert>
        // set attribution delegate
        config.setOnAttributionChangedListener(new OnAttributionChangedListener() {
            @Override
            public void onAttributionChanged(AdjustAttribution attribution) {
                Log.debug("[ADJUST] attribution: " + attribution.toString());
            }
        });

        // set event success tracking delegate
        config.setOnEventTrackingSucceededListener(new OnEventTrackingSucceededListener() {
            @Override
            public void onFinishedEventTrackingSucceeded(AdjustEventSuccess eventSuccessResponseData) {
                Log.debug("[ADJUST] success event tracking: " + eventSuccessResponseData.toString());
            }
        });

        // set event failure tracking delegate
        config.setOnEventTrackingFailedListener(new OnEventTrackingFailedListener() {
            @Override
            public void onFinishedEventTrackingFailed(AdjustEventFailure eventFailureResponseData) {
                Log.debug("[ADJUST] failed event tracking: " + eventFailureResponseData.toString());
            }
        });

        // set session success tracking delegate
        config.setOnSessionTrackingSucceededListener(new OnSessionTrackingSucceededListener() {
            @Override
            public void onFinishedSessionTrackingSucceeded(AdjustSessionSuccess sessionSuccessResponseData) {
                Log.debug("[ADJUST] success session tracking: " + sessionSuccessResponseData.toString());
            }
        });

        // set session failure tracking delegate
        config.setOnSessionTrackingFailedListener(new OnSessionTrackingFailedListener() {
            @Override
            public void onFinishedSessionTrackingFailed(AdjustSessionFailure sessionFailureResponseData) {
                Log.debug("[ADJUST] failed session tracking: " + sessionFailureResponseData.toString());
            }
        });

        // evaluate deeplink to be launched
        config.setOnDeeplinkResponseListener(new OnDeeplinkResponseListener() {
            @Override
            public boolean launchReceivedDeeplink(Uri deeplink) {
                Log.debug("[ADJUST] deeplink to open: " + deeplink);
                return true;
            }
        });

        Adjust.onCreate(config);
		</insert>
	</gameActivityOnCreateAdditions>

	<gameActivityOnResumeAdditions>
		<insert>
		Adjust.onResume();
		</insert>
	</gameActivityOnResumeAdditions>

	<gameActivityOnPauseAdditions>
		<insert>
		Adjust.onPause();
		</insert>
	</gameActivityOnPauseAdditions>
</root>
