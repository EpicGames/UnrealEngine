// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class AndroidAdjust : ModuleRules
	{
		public AndroidAdjust(ReadOnlyTargetRules Target) : base(Target)
        {
			BinariesSubFolder = "NotForLicensees";

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					// ... add other public dependencies that you statically link with here ...
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Analytics",
					"ApplicationCore",
					// ... add private dependencies that you statically link with here ...
				}
			);

			PublicIncludePathModuleNames.AddRange(
				new string[]
				{
					"Analytics",
					"Core",
					"Launch",
				}
			);

			string PluginPath = Utils.MakePathRelativeTo(ModuleDirectory, Target.RelativeEnginePath);
			bool bHasAdjustSDK = true; //  Directory.Exists(System.IO.Path.Combine(PluginPath, "ThirdParty", "adjust_library"));
            if (bHasAdjustSDK)
            {
                PublicDefinitions.Add("WITH_ADJUST=1");
				AdditionalPropertiesForReceipt.Add("AndroidPlugin", Path.Combine(PluginPath, "Adjust_UPL.xml"));
            }
            else
            {
                PublicDefinitions.Add("WITH_ADJUST=0");
            }
        }
	}
}