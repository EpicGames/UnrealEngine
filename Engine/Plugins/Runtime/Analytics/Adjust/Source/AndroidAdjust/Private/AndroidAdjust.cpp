// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidAdjust.h"
#include "Analytics.h"
#include "Modules/ModuleManager.h"
#include "AndroidAdjustProvider.h"
#include "Android/AndroidJNI.h"
#include "Android/AndroidApplication.h"
#include "Misc/Paths.h"
#include "Misc/ConfigCacheIni.h"

IMPLEMENT_MODULE( FAnalyticsAndroidAdjust, AndroidAdjust )

TSharedPtr<IAnalyticsProvider> FAnalyticsProviderAdjust::Provider;

void FAnalyticsAndroidAdjust::StartupModule()
{
}

void FAnalyticsAndroidAdjust::ShutdownModule()
{
	FAnalyticsProviderAdjust::Destroy();
}

// Android JNI to call Adjust UPL injected methods
void AndroidThunkCpp_Adjust_SetEnabled(bool bEnable)
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		static jmethodID Method = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "AndroidThunkJava_Adjust_SetEnabled", "(Z)V", false);
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, Method, bEnable);
	}
}

void AndroidThunkCpp_Adjust_SetOfflineMode(bool bOffline)
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		static jmethodID Method = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "AndroidThunkJava_Adjust_SetOfflineMode", "(Z)V", false);
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, Method, bOffline);
	}
}

void AndroidThunkCpp_Adjust_SetPushToken(const FString& Token)
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		static jmethodID Method = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, " AndroidThunkJava_Adjust_SetPushToken", "(Ljava/lang/String;)V", false);
		auto TokenJava = FJavaHelper::ToJavaString(Env, Token);
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, Method, *TokenJava);
	}
}

void AndroidThunkCpp_Adjust_AddSessionPartnerParameter(const FString& Key, const FString& Value)
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		static jmethodID Method = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "AndroidThunkJava_Adjust_AddSessionPartnerParameter", "(Ljava/lang/String;Ljava/lang/String;)V", false);
		auto KeyJava = FJavaHelper::ToJavaString(Env, Key);
		auto ValueJava = FJavaHelper::ToJavaString(Env, Value);
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, Method, *KeyJava, *ValueJava);
	}
}

void AndroidThunkCpp_Adjust_RemoveSessionPartnerParameter(const FString& Key)
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		static jmethodID Method = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "AndroidThunkJava_Adjust_RemoveSessionPartnerParameter", "(Ljava/lang/String;)V", false);
		auto KeyJava = FJavaHelper::ToJavaString(Env, Key);
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, Method, *KeyJava);
	}
}

void AndroidThunkCpp_Adjust_ResetSessionPartnerParameters()
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		static jmethodID Method = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "AndroidThunkJava_Adjust_ResetSessionPartnerParameters", "()V", false);
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, Method);
	}
}

void AndroidThunkCpp_Adjust_Event_AddCallbackParameter(const FString& Key, const FString& Value)
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		static jmethodID Method = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "AndroidThunkJava_Adjust_Event_AddCallbackParameter", "(Ljava/lang/String;Ljava/lang/String;)V", false);
		auto KeyJava = FJavaHelper::ToJavaString(Env, Key);
		auto ValueJava = FJavaHelper::ToJavaString(Env, Value);
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, Method, *KeyJava, *ValueJava);
	}
}

void AndroidThunkCpp_Adjust_Event_RemoveCallbackParameter(const FString& Key)
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		static jmethodID Method = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "AndroidThunkJava_Adjust_Event_RemoveCallbackParameter", "(Ljava/lang/String;)V", false);
		auto KeyJava = FJavaHelper::ToJavaString(Env, Key);
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, Method, *KeyJava);
	}
}

void AndroidThunkCpp_Adjust_Event_ResetCallbackParameters()
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		static jmethodID Method = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "AndroidThunkJava_Adjust_Event_ResetCallbackParameters", "()V", false);
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, Method);
	}
}

void AndroidThunkCpp_Adjust_Event_AddPartnerParameter(const FString& Key, const FString& Value)
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		static jmethodID Method = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "AndroidThunkJava_Adjust_Event_AddPartnerParameter", "(Ljava/lang/String;Ljava/lang/String;)V", false);
		auto KeyJava = FJavaHelper::ToJavaString(Env, Key);
		auto ValueJava = FJavaHelper::ToJavaString(Env, Value);
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, Method, *KeyJava, *ValueJava);
	}
}

void AndroidThunkCpp_Adjust_Event_RemovePartnerParameter(const FString& Key)
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		static jmethodID Method = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "AndroidThunkJava_Adjust_Event_RemovePartnerParameter", "(Ljava/lang/String;)V", false);
		auto KeyJava = FJavaHelper::ToJavaString(Env, Key);
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, Method, *KeyJava);
	}
}

void AndroidThunkCpp_Adjust_Event_ResetPartnerParameters()
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		static jmethodID Method = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "AndroidThunkJava_Adjust_Event_ResetPartnerParameters", "()V", false);
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, Method);
	}
}

void AndroidThunkCpp_Adjust_SendEvent(const FString& Token)
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		static jmethodID Method = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "AndroidThunkJava_Adjust_SendEvent", "(Ljava/lang/String;)V", false);
		auto TokenJava = FJavaHelper::ToJavaString(Env, Token);
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, Method, *TokenJava);
	}
}

void AndroidThunkCpp_Adjust_SendRevenueEvent(const FString& Token, const FString& OrderId, double Amount, const FString& Currency)
{
	if (JNIEnv* Env = FAndroidApplication::GetJavaEnv())
	{
		static jmethodID Method = FJavaWrapper::FindMethod(Env, FJavaWrapper::GameActivityClassID, "AndroidThunkJava_Adjust_SendRevenueEvent", "(Ljava/lang/String;Ljava/lang/String;DLjava/lang/String;)V", false);
		auto TokenJava = FJavaHelper::ToJavaString(Env, Token);
		auto OrderIdJava = FJavaHelper::ToJavaString(Env, OrderId);
		auto CurrencyJava = FJavaHelper::ToJavaString(Env, Currency);
		FJavaWrapper::CallVoidMethod(Env, FJavaWrapper::GameActivityThis, Method, *TokenJava, *OrderIdJava, Amount, *CurrencyJava);
	}
}

// End Android JNI to call Adjust UPL injected methods

TSharedPtr<IAnalyticsProvider> FAnalyticsAndroidAdjust::CreateAnalyticsProvider(const FAnalyticsProviderConfigurationDelegate& GetConfigValue) const
{
	if (GetConfigValue.IsBound())
	{
		const FString InAppToken = GetConfigValue.Execute(TEXT("AdjustAppToken"), true);
		return FAnalyticsProviderAdjust::Create(InAppToken);
	}
	else
	{
		UE_LOGF(LogAnalytics, Warning, "AndroidAdjust::CreateAnalyticsProvider called with an unbound delegate");
	}
	return nullptr;
}

// Provider

FAnalyticsProviderAdjust::FAnalyticsProviderAdjust(const FString inAppToken) :
	AppToken(inAppToken)
{
#if WITH_ADJUST
	// NOTE: currently expect these events to have been added!
	// SessionAttributes
	// Item Purchase
	// Currency Purchase
	// Currency Given
	// Error
	// Progress

	// add event attributes from ini
	FString IniName = FString::Printf(TEXT("%sDefaultEngine.ini"), *FPaths::SourceConfigDir());
	EventMap.Empty();
	TArray<FString> EventNames;
	TArray<FString> EventTokens;
	int NameCount = GConfig->GetArray(TEXT("AdjustAnalyticsEventMapping"), TEXT("EventNames"), EventNames, IniName);
	int TokenCount = GConfig->GetArray(TEXT("AdjustAnalyticsEventMapping"), TEXT("EventTokens"), EventTokens, IniName);
	int Count = NameCount <= TokenCount ? NameCount : TokenCount;
	for (int Index = 0; Index < Count; ++Index)
	{
		EventMap.Add(EventNames[Index], EventTokens[Index]);
	}

	#else
	UE_LOGF(LogAnalytics, Warning, "WITH_ADJUST=0. Are you missing the SDK?");
#endif
}

FAnalyticsProviderAdjust::~FAnalyticsProviderAdjust()
{
	if (bHasSessionStarted)
	{
		EndSession();
	}
}

bool FAnalyticsProviderAdjust::StartSession(const TArray<FAnalyticsEventAttribute>& Attributes)
{
#if WITH_ADJUST
	const int32 AttrCount = Attributes.Num();

	// add session attributes (this will be on all events)

	for (auto Attr : Attributes)
	{
		AndroidThunkCpp_Adjust_AddSessionPartnerParameter(Attr.GetName(), Attr.GetValue());
	}
	RecordEvent(TEXT("SessionAttributes"), Attributes);

	if (!bHasSessionStarted)
	{
		UE_LOGF(LogAnalytics, Display, "AndroidAdjust::StartSession(%d attributes)", AttrCount);
	}
	else
	{
		UE_LOGF(LogAnalytics, Display, "AndroidAdjust::RestartSession(%d attributes)", AttrCount);
	}
	bHasSessionStarted = true;
	return bHasSessionStarted;
#else
	UE_LOGF(LogAnalytics, Warning, "WITH_ADJUST=0. Are you missing the SDK?");
	return false;
#endif
}

void FAnalyticsProviderAdjust::EndSession()
{
#if WITH_ADJUST
	bHasSessionStarted = false;

	UE_LOGF(LogAnalytics, Display, "AndroidAdjust::EndSession");
#else
	UE_LOGF(LogAnalytics, Warning, "WITH_ADJUST=0. Are you missing the SDK?");
#endif
}

void FAnalyticsProviderAdjust::FlushEvents()
{
#if WITH_ADJUST
	UE_LOGF(LogAnalytics, Display, "AndroidAdjust::FlushEvents");
#else
	UE_LOGF(LogAnalytics, Warning, "WITH_ADJUST=0. Are you missing the SDK?");
#endif
}

void FAnalyticsProviderAdjust::SetUserID(const FString& InUserID)
{
#if WITH_ADJUST
	UserId = InUserID;
	UE_LOGF(LogAnalytics, Display, "AndroidAdjust::SetUserID(%ls)", *UserId);
#else
	UE_LOGF(LogAnalytics, Warning, "WITH_ADJUST=0. Are you missing the SDK?");
#endif
}

FString FAnalyticsProviderAdjust::GetUserID() const
{
#if WITH_ADJUST
	UE_LOGF(LogAnalytics, Display, "AndroidAdjust::GetUserID - returning cached id '%ls'", *UserId);

	return UserId;
#else
	UE_LOGF(LogAnalytics, Warning, "WITH_ADJUST=0. Are you missing the SDK?");
	return FString();
#endif
}

FString FAnalyticsProviderAdjust::GetSessionID() const
{
#if WITH_ADJUST
	FString Id = TEXT("unavailable");

	UE_LOGF(LogAnalytics, Display, "AndroidAdjust::GetSessionID - returning the id as '%ls'", *Id);

	return Id;
#else
	UE_LOGF(LogAnalytics, Warning, "WITH_ADJUST=0. Are you missing the SDK?");
	return FString();
#endif
}

bool FAnalyticsProviderAdjust::SetSessionID(const FString& InSessionID)
{
#if WITH_ADJUST
	// Ignored
	UE_LOGF(LogAnalytics, Display, "AndroidAdjust::SetSessionID - ignoring call");
	return false;
#else
	UE_LOGF(LogAnalytics, Warning, "WITH_ADJUST=0. Are you missing the SDK?");
	return false;
#endif
}

void FAnalyticsProviderAdjust::RecordEvent(const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attributes)
{
#if WITH_ADJUST
	FString* EventTokenRef = EventMap.Find(EventName);

	TArray<FAnalyticsEventAttribute> EventAttributes;
	EventAttributes.Append(Attributes);
	EventAttributes.Append(DefaultEventAttributes);

	if (EventTokenRef != nullptr)
	{
		FString EventToken = *EventTokenRef;

		AndroidThunkCpp_Adjust_Event_ResetCallbackParameters();
		AndroidThunkCpp_Adjust_Event_ResetPartnerParameters();

		const int32 AttrCount = EventAttributes.Num();
		if (AttrCount > 0)
		{
			// add event attributes
			for (auto Attr : EventAttributes)
			{
				AndroidThunkCpp_Adjust_Event_AddCallbackParameter(Attr.GetName(), Attr.GetValue());
			}
		}
		AndroidThunkCpp_Adjust_SendEvent(EventToken);
		UE_LOGF(LogAnalytics, Display, "AndroidAdjust::RecordEvent('%ls', %d attributes) Token=%ls", *EventName, AttrCount, *EventToken);
	}
#else
	UE_LOGF(LogAnalytics, Warning, "WITH_ADJUST=0. Are you missing the SDK?");
#endif
}

void FAnalyticsProviderAdjust::RecordItemPurchase(const FString& ItemId, const FString& Currency, int PerItemCost, int ItemQuantity)
{
#if WITH_ADJUST
	FString* EventTokenRef = EventMap.Find(TEXT("Item Purchase"));
	if (EventTokenRef != nullptr)
	{
		FString EventToken = *EventTokenRef;

		AndroidThunkCpp_Adjust_Event_ResetPartnerParameters();

		AndroidThunkCpp_Adjust_Event_ResetCallbackParameters();
		AndroidThunkCpp_Adjust_Event_AddCallbackParameter(TEXT("ItemId"), ItemId);
		AndroidThunkCpp_Adjust_Event_AddCallbackParameter(TEXT("Currency"), Currency);
		AndroidThunkCpp_Adjust_Event_AddCallbackParameter(TEXT("PerItemCost"), FString::FromInt(PerItemCost));
		AndroidThunkCpp_Adjust_Event_AddCallbackParameter(TEXT("ItemQuantity"), FString::FromInt(ItemQuantity));
	
		// @TODO: This is probably wrong.. might just want to do a normal event and forget about revenue / order id (note: input is in cents so divide by 100)
		AndroidThunkCpp_Adjust_SendRevenueEvent(EventToken, ItemId, PerItemCost * ItemQuantity * 0.01, Currency);

		UE_LOGF(LogAnalytics, Display, "AndroidAdjust::RecordItemPurchase('%ls', '%ls', %d, %d) Token=%ls", *ItemId, *Currency, PerItemCost, ItemQuantity, *EventToken);
	}
#else
	UE_LOGF(LogAnalytics, Warning, "WITH_ADJUST=0. Are you missing the SDK?");
#endif
}

void FAnalyticsProviderAdjust::RecordCurrencyPurchase(const FString& GameCurrencyType, int GameCurrencyAmount, const FString& RealCurrencyType, float RealMoneyCost, const FString& PaymentProvider)
{
#if WITH_ADJUST
	FString* EventTokenRef = EventMap.Find(TEXT("Currency Purchase"));
	if (EventTokenRef != nullptr)
	{
		FString EventToken = *EventTokenRef;

		AndroidThunkCpp_Adjust_Event_ResetPartnerParameters();

		AndroidThunkCpp_Adjust_Event_ResetCallbackParameters();
		AndroidThunkCpp_Adjust_Event_AddCallbackParameter(TEXT("GameCurrencyType"), GameCurrencyType);
		AndroidThunkCpp_Adjust_Event_AddCallbackParameter(TEXT("GameCurrencyAmount"), FString::FromInt(GameCurrencyAmount));
		AndroidThunkCpp_Adjust_Event_AddCallbackParameter(TEXT("RealCurrencyType"), RealCurrencyType);
		AndroidThunkCpp_Adjust_Event_AddCallbackParameter(TEXT("RealMoneyCost"), FString::Printf(TEXT("%.02f"), RealMoneyCost));
		AndroidThunkCpp_Adjust_Event_AddCallbackParameter(TEXT("PaymentProvider"), PaymentProvider);

		// @TODO: This is probably wrong.. might just want to do a normal event and forget about revenue / order id
		AndroidThunkCpp_Adjust_SendRevenueEvent(EventToken, GameCurrencyType, RealMoneyCost, RealCurrencyType);

		UE_LOGF(LogAnalytics, Display, "AndroidAdjust::RecordCurrencyPurchase('%ls', %d, '%ls', %.02f, %ls) Token=%ls", *GameCurrencyType, GameCurrencyAmount, *RealCurrencyType, RealMoneyCost, *PaymentProvider, *EventToken);
	}
#else
	UE_LOGF(LogAnalytics, Warning, "WITH_ADJUST=0. Are you missing the SDK?");
#endif
}

void FAnalyticsProviderAdjust::RecordCurrencyGiven(const FString& GameCurrencyType, int GameCurrencyAmount)
{
#if WITH_ADJUST
	FString* EventTokenRef = EventMap.Find(TEXT("Currency Given"));
	if (EventTokenRef != nullptr)
	{
		FString EventToken = *EventTokenRef;

		AndroidThunkCpp_Adjust_Event_ResetPartnerParameters();

		AndroidThunkCpp_Adjust_Event_ResetCallbackParameters();
		AndroidThunkCpp_Adjust_Event_AddCallbackParameter(TEXT("GameCurrencyType"), GameCurrencyType);
		AndroidThunkCpp_Adjust_Event_AddCallbackParameter(TEXT("GameCurrencyAmount"), FString::FromInt(GameCurrencyAmount));

		AndroidThunkCpp_Adjust_SendEvent(EventToken);

		UE_LOGF(LogAnalytics, Display, "AndroidAdjust::RecordCurrencyGiven('%ls', %d) Token=%ls", *GameCurrencyType, GameCurrencyAmount, *EventToken);
	}
#else
	UE_LOGF(LogAnalytics, Warning, "WITH_ADJUST=0. Are you missing the SDK?");
#endif
}

void FAnalyticsProviderAdjust::RecordError(const FString& Error, const TArray<FAnalyticsEventAttribute>& EventAttrs)
{
#if WITH_ADJUST
	FString* EventTokenRef = EventMap.Find(TEXT("Error"));
	if (EventTokenRef != nullptr)
	{
		FString EventToken = *EventTokenRef;

		AndroidThunkCpp_Adjust_Event_ResetCallbackParameters();
		AndroidThunkCpp_Adjust_Event_ResetPartnerParameters();

		const int32 AttrCount = EventAttrs.Num();
		if (AttrCount > 0)
		{
			// add event attributes
			for (auto Attr : EventAttrs)
			{
				AndroidThunkCpp_Adjust_Event_AddCallbackParameter(Attr.GetName(), Attr.GetValue());
			}
		}

		AndroidThunkCpp_Adjust_SendEvent(EventToken);

		UE_LOGF(LogAnalytics, Display, "AndroidAdjust::RecordError('%ls', %d) Token=%ls", *Error, AttrCount, *EventToken);
	}
#else
	UE_LOGF(LogAnalytics, Warning, "WITH_ADJUST=0. Are you missing the SDK?");
#endif
}

void FAnalyticsProviderAdjust::RecordProgress(const FString& ProgressType, const FString& ProgressHierarchy, const TArray<FAnalyticsEventAttribute>& EventAttrs)
{
#if WITH_ADJUST
	FString* EventTokenRef = EventMap.Find(TEXT("Progress"));
	if (EventTokenRef != nullptr)
	{
		FString EventToken = *EventTokenRef;

		AndroidThunkCpp_Adjust_Event_ResetCallbackParameters();
		AndroidThunkCpp_Adjust_Event_ResetPartnerParameters();

		AndroidThunkCpp_Adjust_Event_AddCallbackParameter(TEXT("ProgressType"), ProgressType);
		AndroidThunkCpp_Adjust_Event_AddCallbackParameter(TEXT("ProgressHierarchy"), ProgressHierarchy);

		const int32 AttrCount = EventAttrs.Num();
		if (AttrCount > 0)
		{
			// add event attributes
			for (auto Attr : EventAttrs)
			{
				AndroidThunkCpp_Adjust_Event_AddCallbackParameter(Attr.GetName(), Attr.GetValue());
			}
		}

		AndroidThunkCpp_Adjust_SendEvent(EventToken);

		UE_LOGF(LogAnalytics, Display, "AndroidAdjust::RecordProgress('%ls', %ls, %d) Token=%ls", *ProgressType, *ProgressHierarchy, AttrCount, *EventToken);
	}
#else
	UE_LOGF(LogAnalytics, Warning, "WITH_ADJUST=0. Are you missing the SDK?");
#endif
}

void FAnalyticsProviderAdjust::SetDefaultEventAttributes(TArray<FAnalyticsEventAttribute>&& Attributes)
{
	DefaultEventAttributes = Attributes;
}

TArray<FAnalyticsEventAttribute> FAnalyticsProviderAdjust::GetDefaultEventAttributesSafe() const
{
	return DefaultEventAttributes;
}

int32 FAnalyticsProviderAdjust::GetDefaultEventAttributeCount() const
{
	return DefaultEventAttributes.Num();
}

FAnalyticsEventAttribute FAnalyticsProviderAdjust::GetDefaultEventAttribute(int AttributeIndex) const
{
	return DefaultEventAttributes[AttributeIndex];
}