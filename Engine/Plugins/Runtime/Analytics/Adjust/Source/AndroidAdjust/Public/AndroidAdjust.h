// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/IAnalyticsProviderModule.h"
#include "Modules/ModuleManager.h"

class IAnalyticsProvider;

/**
 * The public interface to this module
 */
class FAnalyticsAndroidAdjust :
	public IAnalyticsProviderModule
{
	//--------------------------------------------------------------------------
	// Module functionality
	//--------------------------------------------------------------------------
public:
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline FAnalyticsAndroidAdjust& Get()
	{
		return FModuleManager::LoadModuleChecked< FAnalyticsAndroidAdjust >( "AndroidAdjust" );
	}

	//--------------------------------------------------------------------------
	// provider factory functions
	//--------------------------------------------------------------------------
public:
	/**
	 * IAnalyticsProviderModule interface.
	 * Creates the analytics provider given a configuration delegate.
	 * The keys required exactly match the field names in the Config object. 
	 */
	virtual TSharedPtr<IAnalyticsProvider> CreateAnalyticsProvider(const FAnalyticsProviderConfigurationDelegate& GetConfigValue) const override;
	
private:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

