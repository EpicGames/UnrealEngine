// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class IOSAdjust : ModuleRules
	{
		public IOSAdjust(ReadOnlyTargetRules Target) : base(Target)
        {
			bRequiresPlatformSDK = true;
		
			BinariesSubFolder = "NotForLicensees";

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					// ... add other public dependencies that you statically link with here ...
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Analytics",
					// ... add private dependencies that you statically link with here ...
				}
			);

			// Add the Adjust framework
			PublicAdditionalFrameworks.Add( 
				new Framework( 
					"AdjustSdk",													// Framework name
					"../../ThirdPartyFrameworks/AdjustSdk.embeddedframework.zip"	// Zip name
				)
			);

			PublicFrameworks.AddRange(
				new string[] {
//					"AdSupport",	// optional
//					"iAd",			// optional
				}
			);

			PublicIncludePathModuleNames.AddRange(
				new string[]
				{
					"Analytics",
				}
			);

			//string PluginPath = Utils.MakePathRelativeTo(ModuleDirectory, Target.RelativeEnginePath);
			bool bHasAdjustSDK = true; //  Directory.Exists(System.IO.Path.Combine(PluginPath, "ThirdParty", "adjust_library"));
            if (bHasAdjustSDK)
            {
                PublicDefinitions.Add("WITH_ADJUST=1");
            }
            else
            {
                PublicDefinitions.Add("WITH_ADJUST=0");
            }
        }
	}
}