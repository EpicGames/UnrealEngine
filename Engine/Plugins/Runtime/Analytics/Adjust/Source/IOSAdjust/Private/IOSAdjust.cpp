// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOSAdjust.h"
#include "Apple/AppleStringUtils.h"
#include "Analytics.h"
#include "IOSAdjustProvider.h"
#include "Misc/Paths.h"
#include "Misc/ConfigCacheIni.h"

#import <AdjustSdk/Adjust.h>

IMPLEMENT_MODULE( FAnalyticsIOSAdjust, IOSAdjust )

TSharedPtr<IAnalyticsProvider> FAnalyticsProviderAdjust::Provider;

void FAnalyticsIOSAdjust::StartupModule()
{
}

void FAnalyticsIOSAdjust::ShutdownModule()
{
	FAnalyticsProviderAdjust::Destroy();
}

static bool ConvertToBool(const FString& InString, bool bDefault)
{
	if (InString.Len() == 0)
	{
		return bDefault;
	}
	FString Result = InString.ToLower();
	return Result.Equals(TEXT("true")) || Result.Equals(TEXT("yes"));
}

TSharedPtr<IAnalyticsProvider> FAnalyticsIOSAdjust::CreateAnalyticsProvider(const FAnalyticsProviderConfigurationDelegate& GetConfigValue) const
{
	if (GetConfigValue.IsBound())
	{
		const FString InSandboxNondistribution = GetConfigValue.Execute(TEXT("AdjustSandboxNondistribution"), false);
		const FString InSandboxDistribution = GetConfigValue.Execute(TEXT("AdjustSandboxDistribution"), false);
		const FString InAppToken = GetConfigValue.Execute(TEXT("AdjustAppToken"), true);
		const FString InLogLevel = GetConfigValue.Execute(TEXT("AdjustLogLevel"), false);
		const FString InDefaultTracker = GetConfigValue.Execute(TEXT("AdjustDefaultTracker"), false);
		const FString InEventBuffering = GetConfigValue.Execute(TEXT("AdjustEventBuffering"), false);
		const FString InSendInBackground = GetConfigValue.Execute(TEXT("AdjustSendInBackground"), false);
		const FString InDelayStart = GetConfigValue.Execute(TEXT("AdjustDelayStart"), false);

		// @TODO: this is probably a bad assumption
#if UE_BUILD_SHIPPING
		bool bInSandbox = ConvertToBool(InSandboxDistribution, false);
#else
		bool bInSandbox = ConvertToBool(InSandboxNondistribution, true);
#endif
		bool bInEventBuffering = ConvertToBool(InEventBuffering, false);
		bool bInSendInBackground = ConvertToBool(InSendInBackground, false);
		
        float DelayStart = 0.0f;
		if (InDelayStart.Len() > 0)
		{
            DelayStart = FCString::Atof(*InDelayStart);
		}
		
		return FAnalyticsProviderAdjust::Create(InAppToken, bInSandbox, InLogLevel, InDefaultTracker, bInEventBuffering, bInSendInBackground, DelayStart);
	}
	else
	{
		UE_LOGF(LogAnalytics, Warning, "IOSAdjust::CreateAnalyticsProvider called with an unbound delegate");
	}
	return nullptr;
}

// Provider

FAnalyticsProviderAdjust::FAnalyticsProviderAdjust(const FString InAppToken, bool bInSandbox, const FString InLogLevel, const FString InDefaultTracker, bool bInEventBuffering, bool bSendInBackground, float InDelayStart) :
	AppToken(InAppToken)
{
#if WITH_ADJUST
	// NOTE: currently expect these events to have been added!
	// SessionAttributes
	// Item Purchase
	// Currency Purchase
	// Currency Given
	// Error
	// Progress

	// add event attributes from ini
	FString IniName = FString::Printf(TEXT("%sDefaultEngine.ini"), *FPaths::SourceConfigDir());
	EventMap.Empty();
	TArray<FString> EventNames;
	TArray<FString> EventTokens;
	int NameCount = GConfig->GetArray(TEXT("AdjustAnalyticsEventMapping"), TEXT("EventNames"), EventNames, IniName);
	int TokenCount = GConfig->GetArray(TEXT("AdjustAnalyticsEventMapping"), TEXT("EventTokens"), EventTokens, IniName);
	int Count = NameCount <= TokenCount ? NameCount : TokenCount;
	for (int Index = 0; Index < Count; ++Index)
	{
		EventMap.Add(EventNames[Index], EventTokens[Index]);
	}

	// I hope this is the right place to do this (supposed to be in didFinishLaunching in application delegate
	NSString* IOSAppToken = [NSString stringWithFString : InAppToken];
	NSString* Environment = bInSandbox ? ADJEnvironmentSandbox : ADJEnvironmentProduction;
	ADJConfig* adjustConfig;
	
	// Yes, I know SUPRESS is inconsistent, this is from their SDK
	if (InLogLevel.Equals("SUPRESS"))
		adjustConfig = [ADJConfig configWithAppToken:IOSAppToken environment:Environment allowSuppressLogLevel:YES];
	else
		adjustConfig = [ADJConfig configWithAppToken:IOSAppToken environment:Environment];
	
	if (InLogLevel.Equals("VERBOSE"))
		[adjustConfig setLogLevel:ADJLogLevelVerbose];
	else if (InLogLevel.Equals("DEBUG"))
		[adjustConfig setLogLevel:ADJLogLevelDebug];
	else if (InLogLevel.Equals("INFO"))
		[adjustConfig setLogLevel:ADJLogLevelInfo];
	else if (InLogLevel.Equals("WARN"))
		[adjustConfig setLogLevel:ADJLogLevelWarn];
	else if (InLogLevel.Equals("ERROR"))
		[adjustConfig setLogLevel:ADJLogLevelError];
	else if (InLogLevel.Equals("ASSERT"))
		[adjustConfig setLogLevel:ADJLogLevelAssert];
	else if (InLogLevel.Equals("SUPRESS"))
		[adjustConfig setLogLevel:ADJLogLevelSuppress];
	else [adjustConfig setLogLevel:ADJLogLevelInfo];

	if (!InDefaultTracker.IsEmpty())
	{
		[adjustConfig setDefaultTracker:[NSString stringWithFString : InDefaultTracker]];
	}
	
	if (bInEventBuffering)
	{
		[adjustConfig setEventBufferingEnabled:YES];
	}

	if (bSendInBackground)
	{
		[adjustConfig setSendInBackground:YES];
	}
	
	if (InDelayStart > 0.0f)
	{
		[adjustConfig setDelayStart:InDelayStart];
	}
	
	[Adjust appDidLaunch:adjustConfig];
	[Adjust trackSubsessionStart];
	
#else
	UE_LOGF(LogAnalytics, Warning, "WITH_ADJUST=0. Are you missing the SDK?");
#endif
}

FAnalyticsProviderAdjust::~FAnalyticsProviderAdjust()
{
	if (bHasSessionStarted)
	{
		EndSession();
	}
}

bool FAnalyticsProviderAdjust::StartSession(const TArray<FAnalyticsEventAttribute>& Attributes)
{
#if WITH_ADJUST
	const int32 AttrCount = Attributes.Num();

	// add session attributes (this will be on all events)

	for (auto Attr : Attributes)
	{
		NSString* IOSKey = [NSString stringWithFString : Attr.GetName()];
		NSString* IOSValue = [NSString stringWithFString : Attr.GetValue()];
		[Adjust addSessionPartnerParameter:IOSKey value:IOSValue];
	}
	RecordEvent(TEXT("SessionAttributes"), Attributes);

	if (!bHasSessionStarted)
	{
		UE_LOGF(LogAnalytics, Display, "IOSAdjust::StartSession(%d attributes)", AttrCount);
	}
	else
	{
		UE_LOGF(LogAnalytics, Display, "IOSAdjust::RestartSession(%d attributes)", AttrCount);
	}
	bHasSessionStarted = true;
	return bHasSessionStarted;
#else
	UE_LOGF(LogAnalytics, Warning, "WITH_ADJUST=0. Are you missing the SDK?");
	return false;
#endif
}

void FAnalyticsProviderAdjust::EndSession()
{
#if WITH_ADJUST
	bHasSessionStarted = false;

	UE_LOGF(LogAnalytics, Display, "IOSAdjust::EndSession");
#else
	UE_LOGF(LogAnalytics, Warning, "WITH_ADJUST=0. Are you missing the SDK?");
#endif
}

void FAnalyticsProviderAdjust::FlushEvents()
{
#if WITH_ADJUST
	[Adjust sendFirstPackages];
	UE_LOGF(LogAnalytics, Display, "IOSAdjust::FlushEvents");
#else
	UE_LOGF(LogAnalytics, Warning, "WITH_ADJUST=0. Are you missing the SDK?");
#endif
}

void FAnalyticsProviderAdjust::SetUserID(const FString& InUserID)
{
#if WITH_ADJUST
	UserId = InUserID;
	UE_LOGF(LogAnalytics, Display, "IOSAdjust::SetUserID(%ls)", *UserId);
#else
	UE_LOGF(LogAnalytics, Warning, "WITH_ADJUST=0. Are you missing the SDK?");
#endif
}

FString FAnalyticsProviderAdjust::GetUserID() const
{
#if WITH_ADJUST
	UE_LOGF(LogAnalytics, Display, "IOSAdjust::GetUserID - returning cached id '%ls'", *UserId);

	return UserId;
#else
	UE_LOGF(LogAnalytics, Warning, "WITH_ADJUST=0. Are you missing the SDK?");
	return FString();
#endif
}

FString FAnalyticsProviderAdjust::GetSessionID() const
{
#if WITH_ADJUST
	FString Id = TEXT("unavailable");

	UE_LOGF(LogAnalytics, Display, "IOSAdjust::GetSessionID - returning the id as '%ls'", *Id);

	return Id;
#else
	UE_LOGF(LogAnalytics, Warning, "WITH_ADJUST=0. Are you missing the SDK?");
	return FString();
#endif
}

bool FAnalyticsProviderAdjust::SetSessionID(const FString& InSessionID)
{
#if WITH_ADJUST
	// Ignored
	UE_LOGF(LogAnalytics, Display, "IOSAdjust::SetSessionID - ignoring call");
	return false;
#else
	UE_LOGF(LogAnalytics, Warning, "WITH_ADJUST=0. Are you missing the SDK?");
	return false;
#endif
}

void FAnalyticsProviderAdjust::RecordEvent(const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attributes)
{
#if WITH_ADJUST
	FString* EventTokenRef = EventMap.Find(EventName);
	if (EventTokenRef != nullptr)
	{
		TArray<FAnalyticsEventAttribute> EventAttributes;
		EventAttributes.Append(Attributes);
		EventAttributes.Append(DefaultEventAttributes);

		FString EventToken = *EventTokenRef;
		NSString* IOSEventToken = [NSString stringWithFString : EventToken];
		
		ADJEvent *event = [ADJEvent eventWithEventToken:IOSEventToken];
		const int32 AttrCount = EventAttributes.Num();
		if (AttrCount > 0)
		{
			// add event attributes
			for (auto Attr : EventAttributes)
			{
				NSString* IOSKey = [NSString stringWithFString : Attr.GetName()];
				NSString* IOSValue = [NSString stringWithFString : Attr.GetValue()];
				[event addCallbackParameter:IOSKey value:IOSValue];
			}
		}
		[Adjust trackEvent:event];

		UE_LOGF(LogAnalytics, Display, "IOSAdjust::RecordEvent('%ls', %d attributes) Token=%ls", *EventName, AttrCount, *EventToken);
	}
#else
	UE_LOGF(LogAnalytics, Warning, "WITH_ADJUST=0. Are you missing the SDK?");
#endif
}

void FAnalyticsProviderAdjust::RecordItemPurchase(const FString& ItemId, const FString& Currency, int PerItemCost, int ItemQuantity)
{
#if WITH_ADJUST
	FString* EventTokenRef = EventMap.Find(TEXT("Item Purchase"));
	if (EventTokenRef != nullptr)
	{
		FString EventToken = *EventTokenRef;
		NSString* IOSEventToken = [NSString stringWithFString : EventToken];
		
		ADJEvent *event = [ADJEvent eventWithEventToken:IOSEventToken];
		[event addCallbackParameter:@"ItemId" value:[NSString stringWithFString : ItemId]];
		[event addCallbackParameter:@"Currency" value:[NSString stringWithFString : Currency]];
		[event addCallbackParameter:@"PerItemCost" value:[NSString stringWithFormat:@"%d", PerItemCost]];
		[event addCallbackParameter:@"ItemQuantity" value:[NSString stringWithFormat:@"%d", ItemQuantity]];

		// @TODO: This is probably wrong.. might just want to do a normal event and forget about revenue / order id (note: input is in cents so divide by 100)
		[event setRevenue:(PerItemCost * ItemQuantity * 0.01) currency:[NSString stringWithFString : Currency]];
//		[event setTransactionId:[NSString stringWithFString : ItemId]];
		[Adjust trackEvent:event];

		UE_LOGF(LogAnalytics, Display, "IOSAdjust::RecordItemPurchase('%ls', '%ls', %d, %d) Token=%ls", *ItemId, *Currency, PerItemCost, ItemQuantity, *EventToken);
	}
#else
	UE_LOGF(LogAnalytics, Warning, "WITH_ADJUST=0. Are you missing the SDK?");
#endif
}

void FAnalyticsProviderAdjust::RecordCurrencyPurchase(const FString& GameCurrencyType, int GameCurrencyAmount, const FString& RealCurrencyType, float RealMoneyCost, const FString& PaymentProvider)
{
#if WITH_ADJUST
	FString* EventTokenRef = EventMap.Find(TEXT("Currency Purchase"));
	if (EventTokenRef != nullptr)
	{
		FString EventToken = *EventTokenRef;
		NSString* IOSEventToken = [NSString stringWithFString : EventToken];
		
		ADJEvent *event = [ADJEvent eventWithEventToken:IOSEventToken];
		[event addCallbackParameter:@"GameCurrencyType" value:[NSString stringWithFString : GameCurrencyType]];
		[event addCallbackParameter:@"GameCurrencyAmount" value:[NSString stringWithFormat:@"%d", GameCurrencyAmount]];
		[event addCallbackParameter:@"RealCurrencyType" value:[NSString stringWithFString : RealCurrencyType]];
		[event addCallbackParameter:@"RealMoneyCost" value:[NSString stringWithFormat:@"%.02f", RealMoneyCost]];
		[event addCallbackParameter:@"PaymentProvider" value:[NSString stringWithFString : PaymentProvider]];

		[event setRevenue:RealMoneyCost currency:[NSString stringWithFString : RealCurrencyType]];
		[Adjust trackEvent:event];

		UE_LOGF(LogAnalytics, Display, "IOSAdjust::RecordCurrencyPurchase('%ls', %d, '%ls', %.02f, %ls) Token=%ls", *GameCurrencyType, GameCurrencyAmount, *RealCurrencyType, RealMoneyCost, *PaymentProvider, *EventToken);
	}
#else
	UE_LOGF(LogAnalytics, Warning, "WITH_ADJUST=0. Are you missing the SDK?");
#endif
}

void FAnalyticsProviderAdjust::RecordCurrencyGiven(const FString& GameCurrencyType, int GameCurrencyAmount)
{
#if WITH_ADJUST
	FString* EventTokenRef = EventMap.Find(TEXT("Currency Given"));
	if (EventTokenRef != nullptr)
	{
		FString EventToken = *EventTokenRef;
		NSString* IOSEventToken = [NSString stringWithFString : EventToken];
		
		ADJEvent *event = [ADJEvent eventWithEventToken:IOSEventToken];
		[event addCallbackParameter:@"GameCurrencyType" value:[NSString stringWithFString : GameCurrencyType]];
		[event addCallbackParameter:@"GameCurrencyAmount" value:[NSString stringWithFormat:@"%d", GameCurrencyAmount]];

		[Adjust trackEvent:event];

		UE_LOGF(LogAnalytics, Display, "IOSAdjust::RecordCurrencyGiven('%ls', %d) Token=%ls", *GameCurrencyType, GameCurrencyAmount, *EventToken);
	}
#else
	UE_LOGF(LogAnalytics, Warning, "WITH_ADJUST=0. Are you missing the SDK?");
#endif
}

void FAnalyticsProviderAdjust::RecordError(const FString& Error, const TArray<FAnalyticsEventAttribute>& EventAttrs)
{
#if WITH_ADJUST
	FString* EventTokenRef = EventMap.Find(TEXT("Error"));
	if (EventTokenRef != nullptr)
	{
		FString EventToken = *EventTokenRef;
		NSString* IOSEventToken = [NSString stringWithFString : EventToken];
		
		ADJEvent *event = [ADJEvent eventWithEventToken:IOSEventToken];

		const int32 AttrCount = EventAttrs.Num();
		if (AttrCount > 0)
		{
			// add event attributes
			for (auto Attr : EventAttrs)
			{
				NSString* IOSKey = [NSString stringWithFString : Attr.GetName()];
				NSString* IOSValue = [NSString stringWithFString : Attr.GetValue()];
				[event addCallbackParameter:IOSKey value:IOSValue];
			}
		}

		[Adjust trackEvent:event];

		UE_LOGF(LogAnalytics, Display, "AndroidAdjust::RecordError('%ls', %d) Token=%ls", *Error, AttrCount, *EventToken);
	}
#else
	UE_LOGF(LogAnalytics, Warning, "WITH_ADJUST=0. Are you missing the SDK?");
#endif
}

void FAnalyticsProviderAdjust::RecordProgress(const FString& ProgressType, const FString& ProgressHierarchy, const TArray<FAnalyticsEventAttribute>& EventAttrs)
{
#if WITH_ADJUST
	FString* EventTokenRef = EventMap.Find(TEXT("Progress"));
	if (EventTokenRef != nullptr)
	{
		FString EventToken = *EventTokenRef;
		NSString* IOSEventToken = [NSString stringWithFString : EventToken];
		
		ADJEvent *event = [ADJEvent eventWithEventToken:IOSEventToken];

		[event addCallbackParameter:@"ProgressType" value:[NSString stringWithFString : ProgressType]];
		[event addCallbackParameter:@"ProgressHierarchy" value:[NSString stringWithFString : ProgressHierarchy]];

		const int32 AttrCount = EventAttrs.Num();
		if (AttrCount > 0)
		{
			// add event attributes
			for (auto Attr : EventAttrs)
			{
				NSString* IOSKey = [NSString stringWithFString : Attr.GetName()];
				NSString* IOSValue = [NSString stringWithFString : Attr.GetValue()];
				[event addCallbackParameter:IOSKey value:IOSValue];
			}
		}

		[Adjust trackEvent:event];

		UE_LOGF(LogAnalytics, Display, "AndroidAdjust::RecordProgress('%ls', '%ls', %d) Token=%ls", *ProgressType, *ProgressHierarchy, AttrCount, *EventToken);
	}
#else
	UE_LOGF(LogAnalytics, Warning, "WITH_ADJUST=0. Are you missing the SDK?");
#endif
}


void FAnalyticsProviderAdjust::SetDefaultEventAttributes(TArray<FAnalyticsEventAttribute>&& Attributes)
{
	DefaultEventAttributes = Attributes;
}

TArray<FAnalyticsEventAttribute> FAnalyticsProviderAdjust::GetDefaultEventAttributesSafe() const
{
	return DefaultEventAttributes;
}

int32 FAnalyticsProviderAdjust::GetDefaultEventAttributeCount() const
{
	return DefaultEventAttributes.Num();
}

FAnalyticsEventAttribute FAnalyticsProviderAdjust::GetDefaultEventAttribute(int AttributeIndex) const
{
	return DefaultEventAttributes[AttributeIndex];
}