// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/IAnalyticsProvider.h"

class FAnalyticsProviderAdjust :
	public IAnalyticsProvider
{
	/** The AppToken given to you by Adjust dashboard */
	FString AppToken;
	/** Tracks whether we need to start the session or restart it */
	bool bHasSessionStarted;
	/** Cached user id */
	FString UserId;
	/** Event name to token mapping */
	TMap<FString, FString> EventMap;

	/** Singleton for analytics */
	static TSharedPtr<IAnalyticsProvider> Provider;
	FAnalyticsProviderAdjust(const FString InAppToken, bool bInSandbox, const FString InLogLevel, const FString InDefaultTracker, bool bInEventBuffering, bool bSendInBackground, float InDelayStart);

public:
	static TSharedPtr<IAnalyticsProvider> Create(const FString InAppToken, bool bInSandbox, const FString InLogLevel, const FString InDefaultTracker, bool bInEventBuffering, bool bSendInBackground, float InDelayStart)
	{
		if (!Provider.IsValid())
		{
			Provider = TSharedPtr<IAnalyticsProvider>(new FAnalyticsProviderAdjust(InAppToken, bInSandbox,InLogLevel, InDefaultTracker, bInEventBuffering, bSendInBackground, InDelayStart));
		}
		return Provider;
	}
	static void Destroy()
	{
		Provider.Reset();
	}

	virtual ~FAnalyticsProviderAdjust();

	virtual bool StartSession(const TArray<FAnalyticsEventAttribute>& Attributes) override;
	virtual void EndSession() override;
	virtual void FlushEvents() override;

	virtual void SetUserID(const FString& InUserID) override;
	virtual FString GetUserID() const override;

	virtual FString GetSessionID() const override;
	virtual bool SetSessionID(const FString& InSessionID) override;

	virtual void RecordEvent(const FString& EventName, const TArray<FAnalyticsEventAttribute>& Attributes) override;
	virtual void RecordItemPurchase(const FString& ItemId, const FString& Currency, int PerItemCost, int ItemQuantity) override;
	virtual void RecordCurrencyPurchase(const FString& GameCurrencyType, int GameCurrencyAmount, const FString& RealCurrencyType, float RealMoneyCost, const FString& PaymentProvider) override;
	virtual void RecordCurrencyGiven(const FString& GameCurrencyType, int GameCurrencyAmount) override;

	virtual void RecordError(const FString& Error, const TArray<FAnalyticsEventAttribute>& EventAttrs) override;
	virtual void RecordProgress(const FString& ProgressType, const FString& ProgressHierarchy, const TArray<FAnalyticsEventAttribute>& EventAttrs) override;

	virtual void SetDefaultEventAttributes(TArray<FAnalyticsEventAttribute>&& Attributes)  override;
	virtual TArray<FAnalyticsEventAttribute> GetDefaultEventAttributesSafe() const override;
	virtual int32 GetDefaultEventAttributeCount() const  override;
	virtual FAnalyticsEventAttribute GetDefaultEventAttribute(int AttributeIndex) const  override;

private:
	TArray<FAnalyticsEventAttribute>		DefaultEventAttributes;
};
